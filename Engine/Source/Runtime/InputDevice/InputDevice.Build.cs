// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InputDevice : ModuleRules
{
    public InputDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange( new string[] { "Core", "Engine" } );
	}
}
