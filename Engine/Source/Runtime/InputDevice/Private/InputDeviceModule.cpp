// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "IInputDevice.h"
#include "IInputDeviceModule.h"

class FInputDeviceModule : public IInputDeviceModule
{
	virtual TSharedPtr< class IInputDevice > CreateInputDevice(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler)
	{
		TSharedPtr<IInputDevice> DummyVal = NULL;
		return DummyVal;
	}
};

IMPLEMENT_MODULE( FInputDeviceModule, InputDevice );
