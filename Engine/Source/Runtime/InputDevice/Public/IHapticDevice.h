// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FHapticFeedbackValues;

class IHapticDevice
{
public:
	/**
	 * Sets the frequency / amplitude of the haptic channel of a controller.
	 *
	 * @param ControllerId	Id of the controller to set haptic feedback parameters on
	 * @param Hand			Which hand, if supported, to add the haptic feedback to (corresponds to EControllerHand)
	 * @param Values		The values to haptic parameter values (e.g. frequency and amplitude) set for the device
	 */
	virtual void SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values) = 0;

	/**
	 * Determines the valid range of frequencies this haptic device supports, to limit input ranges from UHapticFeedbackEffects
	 */
	virtual void GetHapticFrequencyRange(float& MinFrequency, float& MaxFrequency) const = 0;

	/**
	 * Returns the scaling factor to map the amplitude of UHapticFeedbackEvents from [0.0, 1.0] to the actual range handled by the device
	 */
	virtual float GetHapticAmplitudeScale() const = 0;
};
