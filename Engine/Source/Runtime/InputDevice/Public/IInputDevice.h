// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

struct FForceFeedbackValues;
enum class FForceFeedbackChannelType;
struct FInputDeviceProperty;

/**
 * Governs which thread is allowed to call Tick() and SendControllerEvents() on an IInputDevice.
 * Output methods (SetChannelValue, SetChannelValues, SetLightColor, SetDeviceProperty, haptics, etc.)
 * are always called from the game thread regardless of this affinity declaration.
 *
 * Default is GameThreadOnly — existing third-party IInputDevice plugins require no changes.
 */
enum class EInputDeviceThreadAffinity : uint8
{
	/** Device must be ticked by the game thread. Safe for all legacy and Win32 message-pump devices. */
	GameThreadOnly,

	/** Device may be ticked by the dedicated input thread. Tick() and SendControllerEvents() must be thread-safe. */
	InputThreadSafe,
};

/**
 * Input device interface.
 * Useful for plugins/modules to support custom external input devices.
 */
class IInputDevice
{
public:
	virtual ~IInputDevice() = default;

	/** Tick the interface (e.g. check for new controllers) */
	virtual void Tick( float DeltaTime ) = 0;

	/** Poll for controller state and send events if needed */
	virtual void SendControllerEvents() = 0;

	/**
	 * Returns the thread affinity for this input device — which thread is permitted to call
	 * Tick() and SendControllerEvents(). Defaults to GameThreadOnly so existing devices
	 * require no changes. Override and return InputThreadSafe only if both methods are
	 * fully thread-safe (no game-thread-only APIs, no UObjects, no Slate calls).
	 */
	virtual EInputDeviceThreadAffinity GetThreadAffinity() const { return EInputDeviceThreadAffinity::GameThreadOnly; }

	/** Set which MessageHandler will get the events from SendControllerEvents. */
	virtual void SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler ) = 0;

	/** Exec handler to allow console commands to be passed through for debugging */
    virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) = 0;

	/**
	 * Force Feedback pass through functions
	 */
	virtual void SetChannelValue (int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) = 0;
	virtual void SetChannelValues (int32 ControllerId, const FForceFeedbackValues &values) = 0;
	virtual bool SupportsForceFeedback(int32 ControllerId) { return true; }

	/**
	 * Pass though functions for light color
	 */
	virtual void SetLightColor(int32 ControllerId, FColor Color) { };
	virtual void ResetLightColor(int32 ControllerId) { };

	/**
	* Sets a property for a given controller id.
	* Will be ignored for devices which don't support the property.
	*
	* @param ControllerId the id of the controller whose property is to be applied
	* @param Property Base class pointer to property that will be applied
	*/
	virtual void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property) {}

	/** If this device supports a haptic interface, implement this, and inherit the IHapticDevice interface */
	virtual class IHapticDevice* GetHapticDevice()
	{
		return nullptr;
	}

	virtual bool IsGamepadAttached() const
	{
		return false;
	}
};

