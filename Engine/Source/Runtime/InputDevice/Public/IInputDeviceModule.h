// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "IInputDevice.h"

/** Input parameters to device creation. */
struct FInputDeviceCreationParameters
{
	/** Indicates if this device is operating as a primary device and thus a part of game input system. */
	bool bInitAsPrimaryDevice = true;
};

/**
 * The public interface of the InputDeviceModule
 */
class IInputDeviceModule : public IModuleInterface, public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("InputDevice");
		return FeatureName;
	}

	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature( GetModularFeatureName(), this );
	}

	/**
	 * Singleton-like access to IInputDeviceModule
	 *
	 * @return Returns IInputDeviceModule singleton instance, loading the module on demand if needed
	 */
	static inline IInputDeviceModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IInputDeviceModule >( "InputDevice" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "InputDevice" );
	}

	/**
	 * Calls the normal "CreateInputDevice" logic only if this input device module is preferred.
	 *
	 * Returns null if this device is not preferred
	 */
	TSharedPtr<IInputDevice> CreateInputDeviceIfPreferred(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
	{
		if (!FGenericPlatformMisc::IsPreferredInputDevice(GetPreferredDeviceAPIString()))
		{
			return nullptr;
		}
		
		return CreateInputDevice(InMessageHandler);
	}

	/**
	 * Attempts to create a new input device with the given message handler.  This version will eventually be deprecated and users should
	 * migrate to the new version with an additional parameters struct.
	 *
	 * @return	Interface to the new input device, if we were able to successfully create one
	 */
	virtual TSharedPtr<IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) = 0;

	/**
	 * Attempts to create a new input device interface with the specified input
	 * device creation parameters. This override is for advanced use cases where
	 * users would like to create devices that are not a part of the input
	 * system.
	 *
	 * @return	Interface to the new input device, if we were able to successfully create one
	 */
	virtual TSharedPtr<IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, FInputDeviceCreationParameters InParams)
	{
		// If this is a primary device and it should be constructed for the input system then call the default implementation.
		if (InParams.bInitAsPrimaryDevice)
		{
			return CreateInputDevice(InMessageHandler);
		}

		return nullptr;
	}

protected:
	
	/**
	 * Override this function if you would like your IInputDeviceModule to support
	 * "Preferred Device API" settings.
	 * 
	 * @return Name of the input API as it would appear in the list of preferred input API names in the 
	 * user settings. If null, then this input device module will never be checked for if the user prefers
	 * it and will always be created.
	 */
	virtual const TCHAR* GetPreferredDeviceAPIString() const
	{
		return nullptr;
	}
};
